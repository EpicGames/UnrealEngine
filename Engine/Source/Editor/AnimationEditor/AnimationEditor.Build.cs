// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimationEditor : ModuleRules
{
	public AnimationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
                
				"EditorFramework",
                "UnrealEd",
                "Persona",
                "SkeletonEditor",
                "Kismet",
                "AnimGraph",
				"ToolMenus",
				"TypedElementRuntime"
            }
		);

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "PropertyEditor",
                "SequenceRecorder",
            }
        );

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Persona",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "PropertyEditor",
                "SequenceRecorder",
            }
        );
    }
}
