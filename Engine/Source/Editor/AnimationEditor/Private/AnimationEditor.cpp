// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditor.h"

#include "Algo/Transform.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimCompositeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimationAsset.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "AnimationEditorCommands.h"
#include "AnimationEditorMode.h"
#include "AnimationEditorUtils.h"
#include "AnimationToolMenuContext.h"
#include "DetailLayoutBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Curves/RichCurve.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorModeManager.h"
#include "EditorReimportHandler.h"
#include "Engine/CurveTable.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IAnimSequenceCurveEditor.h"
#include "IAnimationEditorModule.h"
#include "IAnimationSequenceBrowser.h"
#include "IAssetFamily.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "ISequenceRecorder.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "Internationalization/Internationalization.h"
#include "IPersonaViewport.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PersonaCommonCommands.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "PersonaToolMenuContext.h"
#include "Sound/SoundWave.h"
#include "Styling/AppStyle.h"
#include "Subsystems/ImportSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenuOwner.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"

class ITimeSliderController;
class IToolkitHost;
class SWidget;
class UFactory;

const FName AnimationEditorAppIdentifier = FName(TEXT("AnimationEditorApp"));

const FName AnimationEditorModes::AnimationEditorMode(TEXT("AnimationEditorMode"));

const FName AnimationEditorTabs::DetailsTab(TEXT("DetailsTab"));
const FName AnimationEditorTabs::SkeletonTreeTab(TEXT("SkeletonTreeView"));
const FName AnimationEditorTabs::ViewportTab(TEXT("Viewport"));
const FName AnimationEditorTabs::AdvancedPreviewTab(TEXT("AdvancedPreviewTab"));
const FName AnimationEditorTabs::DocumentTab(TEXT("Document"));
const FName AnimationEditorTabs::CurveEditorTab(TEXT("CurveEditor"));
const FName AnimationEditorTabs::AssetBrowserTab(TEXT("SequenceBrowser"));
const FName AnimationEditorTabs::AssetDetailsTab(TEXT("AnimAssetPropertiesTab"));
const FName AnimationEditorTabs::CurveNamesTab(TEXT("AnimCurveViewerTab"));
const FName AnimationEditorTabs::SlotNamesTab(TEXT("SkeletonSlotNames"));
const FName AnimationEditorTabs::AnimMontageSectionsTab(TEXT("AnimMontageSections"));
const FName AnimationEditorTabs::FindReplaceTab(TEXT("FindReplaceTab"));

DEFINE_LOG_CATEGORY(LogAnimationEditor);

#define LOCTEXT_NAMESPACE "AnimationEditor"

FAnimationEditor::~FAnimationEditor()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	//Make sure all delegate for preview mesh change are removed, by setting it to nullptr
	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

void FAnimationEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_AnimationEditor", "Animation Editor"));

	FAssetEditorToolkit::RegisterTabSpawners( InTabManager );
}

void FAnimationEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FAnimationEditor::InitAnimationEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimationAsset* InAnimationAsset)
{
	AnimationAsset = InAnimationAsset;

	// Register post import callback to catch animation imports when we have the asset open (we need to reinit)
	FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FAnimationEditor::HandlePostReimport);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FAnimationEditor::HandlePostImport);

	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FAnimationEditor::HandleOnPreviewSceneSettingsCustomized);
	
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InAnimationAsset, PersonaToolkitArgs);

	PersonaToolkit->GetPreviewScene()->SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode::Animation);

	FSkeletonTreeArgs SkeletonTreeArgs;
	SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(this, &FAnimationEditor::HandleSelectionChanged);
	SkeletonTreeArgs.PreviewScene = PersonaToolkit->GetPreviewScene();
	SkeletonTreeArgs.ContextName = GetToolkitFName();

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, AnimationEditorAppIdentifier, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InAnimationAsset);

	BindCommands();

	AddApplicationMode(
		AnimationEditorModes::AnimationEditorMode,
		MakeShareable(new FAnimationEditorMode(SharedThis(this), SkeletonTree.ToSharedRef())));

	SetCurrentMode(AnimationEditorModes::AnimationEditorMode);

	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	OpenNewAnimationDocumentTab(AnimationAsset);

	PersonaToolkit->GetPreviewScene()->SetAllowMeshHitProxies(false);
}

FName FAnimationEditor::GetToolkitFName() const
{
	return FName("AnimationEditor");
}

FText FAnimationEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "AnimationEditor");
}

FString FAnimationEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "AnimationEditor ").ToString();
}

FLinearColor FAnimationEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FAnimationEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UAnimationToolMenuContext* AnimationToolMenuContext = NewObject<UAnimationToolMenuContext>();
	AnimationToolMenuContext->AnimationEditor = SharedThis(this);
	MenuContext.AddObject(AnimationToolMenuContext);

	UPersonaToolMenuContext* PersonaToolMenuContext = NewObject<UPersonaToolMenuContext>();
	PersonaToolMenuContext->SetToolkit(GetPersonaToolkit());
	MenuContext.AddObject(PersonaToolMenuContext);
}

void FAnimationEditor::Tick(float DeltaTime)
{
	//Do not tick the animation editor if we are compiling the skeletalmesh we edit
	if (GetPersonaToolkit()->GetMesh() && GetPersonaToolkit()->GetMesh()->IsCompiling())
	{
		return;
	}
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

TStatId FAnimationEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimationEditor, STATGROUP_Tickables);
}

void FAnimationEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(AnimationAsset);
}

void FAnimationEditor::BindCommands()
{
	FAnimationEditorCommands::Register();
	
	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().ApplyCompression,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnApplyCompression),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().SetKey,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnSetKey),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::CanSetKey));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().ReimportAnimation,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnReimportAnimation, false),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().ReimportAnimationWithDialog,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnReimportAnimation, true),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().ExportToFBX_AnimData,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnExportToFBX, EExportSourceOption::CurrentAnimation_AnimData),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().ExportToFBX_PreviewMesh,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnExportToFBX, EExportSourceOption::CurrentAnimation_PreviewMesh),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().AddLoopingInterpolation,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnAddLoopingInterpolation),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FAnimationEditorCommands::Get().RemoveBoneTracks,
		FExecuteAction::CreateSP(this, &FAnimationEditor::OnRemoveBoneTrack),
		FCanExecuteAction::CreateSP(this, &FAnimationEditor::HasValidAnimationSequence));

	ToolkitCommands->MapAction(FPersonaCommonCommands::Get().TogglePlay,
		FExecuteAction::CreateRaw(&GetPersonaToolkit()->GetPreviewScene().Get(), &IPersonaPreviewScene::TogglePlayback));
}

TSharedPtr<FAnimationEditor> FAnimationEditor::GetAnimationEditor(const FToolMenuContext& InMenuContext)
{
	if (UAnimationToolMenuContext* Context = InMenuContext.FindContext<UAnimationToolMenuContext>())
	{
		if (Context->AnimationEditor.IsValid())
		{
			return StaticCastSharedPtr<FAnimationEditor>(Context->AnimationEditor.Pin());
		}
	}

	return TSharedPtr<FAnimationEditor>();
}

void FAnimationEditor::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const
{
	DetailBuilder.HideCategory("Animation Blueprint");
}

void FAnimationEditor::ExtendToolbar()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	// Add in Editor Specific functionality
	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	{
		ToolMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			TSharedPtr<FAnimationEditor> AnimationEditor = GetAnimationEditor(InToolMenu->Context);
			if (AnimationEditor.IsValid() && AnimationEditor->PersonaToolkit.IsValid())
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
				FPersonaModule::FCommonToolbarExtensionArgs Args;
				Args.bPreviewAnimation = false;
				Args.bReferencePose = false;
				PersonaModule.AddCommonToolbarExtensions(InToolMenu, Args);
			}
		}), SectionInsertLocation);
	}

	{
		FToolMenuSection& AnimationSection = ToolMenu->AddSection("Animation", LOCTEXT("ToolbarAnimationSectionLabel", "Animation"), SectionInsertLocation);
		AnimationSection.AddEntry(FToolMenuEntry::InitToolBarButton(FAnimationEditorCommands::Get().ReimportAnimation));
		AnimationSection.AddEntry(FToolMenuEntry::InitToolBarButton(FAnimationEditorCommands::Get().ReimportAnimationWithDialog));
		AnimationSection.AddEntry(FToolMenuEntry::InitToolBarButton(FAnimationEditorCommands::Get().ApplyCompression, LOCTEXT("Toolbar_ApplyCompression", "Apply Compression")));
		AnimationSection.AddEntry(FToolMenuEntry::InitComboButton(
			"ExportAsset",
			FToolUIActionChoice(FUIAction()),
			FNewToolMenuChoice(FOnGetContent::CreateSP(this, &FAnimationEditor::GenerateExportAssetMenu)),
			LOCTEXT("ExportAsset_Label", "Export Asset"),
			LOCTEXT("ExportAsset_ToolTip", "Export Assets for this skeleton."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ExportToFBX")
		));
	}

	{
		FToolMenuSection& EditingSection = ToolMenu->AddSection("Editing", LOCTEXT("ToolbarEditingSectionLabel", "Editing"), SectionInsertLocation);
		EditingSection.AddEntry(FToolMenuEntry::InitToolBarButton(FAnimationEditorCommands::Get().SetKey, LOCTEXT("Toolbar_SetKey", "Key")));
	}

	// If the ToolbarExtender is valid, remove it before rebuilding it
	if (ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	IAnimationEditorModule& AnimationEditorModule = FModuleManager::GetModuleChecked<IAnimationEditorModule>("AnimationEditor");
	AddToolbarExtender(AnimationEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TArray<IAnimationEditorModule::FAnimationEditorToolbarExtender> ToolbarExtenderDelegates = AnimationEditorModule.GetAllAnimationEditorToolbarExtenders();

	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if (ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
		}
	}

	// extend extra menu/toolbars
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
		{
			FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
			TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(AnimationAsset);
			AddToolbarWidget(PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily));
		}	
	));
}

void FAnimationEditor::ExtendMenu()
{
	MenuExtender = MakeShareable(new FExtender);

	FToolMenuOwnerScoped OwnerScoped(this);

	// Add in Editor Specific functionality
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("AssetEditor.AnimationEditor.MainMenu.Asset");
	const FToolMenuInsert SectionInsertLocation("AssetEditorActions", EToolMenuInsertType::After);

	FToolMenuSection& AnimationSection = ToolMenu->AddSection("AnimationEditor", LOCTEXT("AnimationEditorAssetMenu_Animation", "Animation"), SectionInsertLocation);
	AnimationSection.AddEntry(FToolMenuEntry::InitMenuEntry(FAnimationEditorCommands::Get().ApplyCompression));
	
	AnimationSection.AddEntry(FToolMenuEntry::InitSubMenu(
		"ExportAsset",
		LOCTEXT("ExportAsset_Label", "Export Asset"),
		LOCTEXT("ExportAsset_ToolTip", "Export Assets for this skeleton."),
		FNewToolMenuChoice(FNewMenuDelegate::CreateSP(this, &FAnimationEditor::FillExportAssetMenu)),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ExportToFBX")
	));

	AnimationSection.AddEntry(FToolMenuEntry::InitMenuEntry(FAnimationEditorCommands::Get().AddLoopingInterpolation));
	AnimationSection.AddEntry(FToolMenuEntry::InitMenuEntry(FAnimationEditorCommands::Get().RemoveBoneTracks));

	AddMenuExtender(MenuExtender);

	IAnimationEditorModule& AnimationEditorModule = FModuleManager::GetModuleChecked<IAnimationEditorModule>("AnimationEditor");
	AddMenuExtender(AnimationEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FAnimationEditor::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObjects(InObjects);

		OnAnimationEditorObjectsSelectedDelegate.Broadcast(InObjects);
	}
}

void FAnimationEditor::HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	RouteTreeSelectionToModeManager(InSelectedItems);

	if (DetailsView.IsValid())
	{
		TArray<UObject*> Objects;
		Algo::TransformIf(InSelectedItems, Objects, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });
		DetailsView->SetObjects(Objects);

		OnAnimationEditorObjectsSelectedDelegate.Broadcast(Objects);
	}
}

void FAnimationEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	if (Viewport.IsValid())
	{
		Viewport->AddOverlayWidget(InOverlaidWidget, ZOrder);
	}
}

void FAnimationEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
{
	if (Viewport.IsValid())
	{
		Viewport->RemoveOverlayWidget(InOverlaidWidget);
	}
}

void FAnimationEditor::HandleObjectSelected(UObject* InObject)
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(InObject);
		
		OnAnimationEditorObjectsSelectedDelegate.Broadcast({InObject});
	}
}

void FAnimationEditor::HandleDetailsCreated(const TSharedRef<IDetailsView>& InDetailsView)
{
	DetailsView = InDetailsView;
}

TSharedPtr<SDockTab> FAnimationEditor::OpenNewAnimationDocumentTab(UAnimationAsset* InAnimAsset)
{
	TSharedPtr<SDockTab> OpenedTab;

	if (InAnimAsset != nullptr)
	{
		FString	DocumentLink;

		FAnimDocumentArgs Args(PersonaToolkit->GetPreviewScene(), GetPersonaToolkit(), GetSkeletonTree()->GetEditableSkeleton(), OnSectionsChanged);
		Args.OnDespatchObjectsSelected = FOnObjectsSelected::CreateSP(this, &FAnimationEditor::HandleObjectsSelected);
		Args.OnDespatchInvokeTab = FOnInvokeTab::CreateSP(this, &FAssetEditorToolkit::InvokeTab);
		Args.OnDespatchSectionsChanged = FSimpleDelegate::CreateSP(this, &FAnimationEditor::HandleSectionsChanged);

		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		TSharedRef<SWidget> TabContents = PersonaModule.CreateEditorWidgetForAnimDocument(SharedThis(this), InAnimAsset, Args, DocumentLink);

		if (AnimationAsset)
		{
			RemoveEditingObject(AnimationAsset);
		}

		AddEditingObject(InAnimAsset);
		AnimationAsset = InAnimAsset;

		GetPersonaToolkit()->GetPreviewScene()->SetPreviewAnimationAsset(InAnimAsset);
		GetPersonaToolkit()->SetAnimationAsset(InAnimAsset);

		// Close existing opened curve tab
		if(AnimCurveDocumentTab.IsValid())
		{
			AnimCurveDocumentTab.Pin()->RequestCloseTab();
		}

		AnimCurveDocumentTab.Reset();

		struct Local
		{
			static FText GetObjectName(UObject* Object)
			{
				return FText::FromString(Object->GetName());
			}
		};

		TAttribute<FText> NameAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetObjectName, (UObject*)InAnimAsset));

		const bool bIsReusedEditor = SharedAnimDocumentTab.IsValid();
		if (bIsReusedEditor)
		{
			OpenedTab = SharedAnimDocumentTab.Pin();
			OpenedTab->SetContent(TabContents);
			OpenedTab->ActivateInParent(ETabActivationCause::SetDirectly);
			OpenedTab->SetLabel(NameAttribute);
			OpenedTab->SetLeftContent(IDocumentation::Get()->CreateAnchor(DocumentLink));
		}
		else
		{
			OpenedTab = SNew(SDockTab)
				.Label(NameAttribute)
				.TabRole(ETabRole::DocumentTab)
				.TabColorScale(GetTabColorScale())
				.OnTabClosed_Lambda([this](TSharedRef<SDockTab> InTab)
				{
					TSharedPtr<SDockTab> CurveTab = AnimCurveDocumentTab.Pin();
					if(CurveTab.IsValid())
					{
						CurveTab->RequestCloseTab();
					}
				})
				[
					TabContents
				];

			OpenedTab->SetLeftContent(IDocumentation::Get()->CreateAnchor(DocumentLink));

			TabManager->InsertNewDocumentTab(AnimationEditorTabs::DocumentTab, FTabManager::ESearchPreference::RequireClosedTab, OpenedTab.ToSharedRef());

			SharedAnimDocumentTab = OpenedTab;
		}

		// Invoke the montage sections tab, and make sure the asset browser is there and in focus when we are dealing with a montage.
		if(InAnimAsset->IsA<UAnimMontage>())
		{
			TabManager->TryInvokeTab(AnimationEditorTabs::AnimMontageSectionsTab);

			// Only activate the asset browser tab when this is a reused Animation Editor window.
			if (bIsReusedEditor)
			{
				TabManager->TryInvokeTab(AnimationEditorTabs::AssetBrowserTab);
			}
			OnSectionsChanged.Broadcast();
		}
		else
		{
			// Close existing opened montage sections tab
			TSharedPtr<SDockTab> OpenMontageSectionsTab = TabManager->FindExistingLiveTab(AnimationEditorTabs::AnimMontageSectionsTab);
			if(OpenMontageSectionsTab.IsValid())
			{
				OpenMontageSectionsTab->RequestCloseTab();
			}	
		}

		if (SequenceBrowser.IsValid())
		{
			SequenceBrowser.Pin()->SelectAsset(InAnimAsset);
		}

		// let the asset family know too
		PersonaModule.RecordAssetOpened(FAssetData(InAnimAsset));
	}

	return OpenedTab;
}

void FAnimationEditor::EditCurves(UAnimSequenceBase* InAnimSequence, const TArray<FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");

	if(!AnimCurveDocumentTab.IsValid())
	{
		TSharedRef<IAnimSequenceCurveEditor> NewCurveEditor = PersonaModule.CreateCurveWidgetForAnimDocument(SharedThis(this), GetPersonaToolkit()->GetPreviewScene(), InAnimSequence, InExternalTimeSliderController, TabManager);
		CurveEditor = NewCurveEditor;

		TSharedPtr<SDockTab> CurveTab = SNew(SDockTab)
			.Label(LOCTEXT("CurveEditorTabTitle", "Curve Editor"))
			.TabRole(ETabRole::DocumentTab)
			.TabColorScale(GetTabColorScale())
			[
				NewCurveEditor
			];

		AnimCurveDocumentTab = CurveTab;

		TabManager->InsertNewDocumentTab(AnimationEditorTabs::CurveEditorTab, FTabManager::ESearchPreference::RequireClosedTab, CurveTab.ToSharedRef());
	}
	else
	{
		TabManager->DrawAttention(AnimCurveDocumentTab.Pin().ToSharedRef());
	}

	check(CurveEditor.IsValid());

	for(const FCurveEditInfo& CurveInfo : InCurveInfo)
	{
		CurveEditor.Pin()->AddCurve(CurveInfo.CurveDisplayName, CurveInfo.CurveColor, CurveInfo.CurveName, CurveInfo.Type, CurveInfo.CurveIndex, CurveInfo.OnCurveModified);
	}

	CurveEditor.Pin()->ZoomToFit();
}

void FAnimationEditor::StopEditingCurves(const TArray<FCurveEditInfo>& InCurveInfo)
{
	if(CurveEditor.IsValid())
	{
		for(const FCurveEditInfo& CurveInfo : InCurveInfo)
		{
			CurveEditor.Pin()->RemoveCurve(CurveInfo.CurveName, CurveInfo.Type, CurveInfo.CurveIndex);
		}
	}
}

void FAnimationEditor::HandleSectionsChanged()
{
	OnSectionsChanged.Broadcast();
}

void FAnimationEditor::SetAnimationAsset(UAnimationAsset* AnimAsset)
{
	HandleOpenNewAsset(AnimAsset);
}

IAnimationSequenceBrowser* FAnimationEditor::GetAssetBrowser() const
{
	return SequenceBrowser.Pin().Get();
}

void FAnimationEditor::HandleOpenNewAsset(UObject* InNewAsset)
{
	if (UAnimationAsset* NewAnimationAsset = Cast<UAnimationAsset>(InNewAsset))
	{
		OpenNewAnimationDocumentTab(NewAnimationAsset);
	}
}

UObject* FAnimationEditor::HandleGetAsset()
{
	return GetEditingObject();
}

bool FAnimationEditor::HasValidAnimationSequence() const
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
	return (AnimSequence != nullptr);
}

bool FAnimationEditor::CanSetKey() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
	return (HasValidAnimationSequence() && PreviewMeshComponent->BonesOfInterest.Num() > 0);
}

void FAnimationEditor::OnSetKey()
{
	if (AnimationAsset)
	{
		UDebugSkelMeshComponent* Component = PersonaToolkit->GetPreviewMeshComponent();
		Component->PreviewInstance->SetKey();
	}
}

void FAnimationEditor::OnReimportAnimation(bool bWithDialog)
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
	if (AnimSequence)
	{
		constexpr bool bAskForNewFileIfMissingTrue = true;
		constexpr bool bShowNotificationTrue = true;
		const FString PreferredReimportFileEmpty = TEXT("");
		constexpr FReimportHandler* SpecifiedReimportHandlerNull = nullptr;
		constexpr bool bForceNewFileFalse = false;
		constexpr int32 SourceFileIndex = INDEX_NONE;
		constexpr bool bAutomatedFalse = false;

		FReimportManager::Instance()->ReimportAsync(AnimSequence
			, bAskForNewFileIfMissingTrue
			, bShowNotificationTrue
			, PreferredReimportFileEmpty
			, SpecifiedReimportHandlerNull
			, SourceFileIndex
			, bForceNewFileFalse
			, bAutomatedFalse
			, bWithDialog);
	}
}

void FAnimationEditor::OnApplyCompression()
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
	if (AnimSequence)
	{
		TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
		AnimSequences.Add(AnimSequence);
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.ApplyCompression(AnimSequences, false);
	}
}

void FAnimationEditor::OnExportToFBX(const EExportSourceOption Option)
{
	UAnimSequence* AnimSequenceToRecord = nullptr;
	if (Option == EExportSourceOption::CurrentAnimation_AnimData)
	{
		TArray<UObject*> AssetsToExport;
		AssetsToExport.Add(AnimationAsset);
		ExportToFBX(AssetsToExport, false);
	}
	else if (Option == EExportSourceOption::CurrentAnimation_PreviewMesh)
	{
		TArray<TSoftObjectPtr<UObject>> Skeletons;
		Skeletons.Add(PersonaToolkit->GetSkeleton());
		AnimationEditorUtils::CreateAnimationAssets(Skeletons, UAnimSequence::StaticClass(), FString("_PreviewMesh"), FAnimAssetCreated::CreateSP(this, &FAnimationEditor::ExportToFBX, true), AnimationAsset, true);
	}
	else
	{
		ensure(false);
	}
}

bool FAnimationEditor::ExportToFBX(const TArray<UObject*> AssetsToExport, bool bRecordAnimation)
{
	bool AnimSequenceExportResult = false;
	TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
	if (AssetsToExport.Num() > 0)
	{
		UAnimSequence* AnimationToRecord = Cast<UAnimSequence>(AssetsToExport[0]);
		if (AnimationToRecord)
		{
			if (bRecordAnimation)
			{
				USkeletalMeshComponent* MeshComponent = PersonaToolkit->GetPreviewMeshComponent();
				RecordMeshToAnimation(MeshComponent, AnimationToRecord);
			}

			AnimSequences.Add(AnimationToRecord);
		}
	}

	if (AnimSequences.Num() > 0)
	{
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		
		
		AnimSequenceExportResult = PersonaModule.ExportToFBX(AnimSequences, GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset());
	}
	return AnimSequenceExportResult;
}

void FAnimationEditor::OnAddLoopingInterpolation()
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
	if (AnimSequence)
	{
		TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
		AnimSequences.Add(AnimSequence);
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.AddLoopingInterpolation(AnimSequences);
	}
}

void FAnimationEditor::OnRemoveBoneTrack()
{
	if ( FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("WarningOnRemovingBoneTracks", "This will clear all bone transform of the animation, source data, and edited layer information. This doesn't remove notifies, and curves. Do you want to continue?")) == EAppReturnType::Yes)
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset);
		if (AnimSequence)
		{
			IAnimationDataController& Controller = AnimSequence->GetController();
			IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("OnRemoveBoneTrack_Bracket", "Removing all Bone Animation and Transform Curve Tracks"));

			Controller.RemoveAllBoneTracks();
			Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform);			
		}
	}
}

TSharedRef< SWidget > FAnimationEditor::GenerateExportAssetMenu() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());
	FillExportAssetMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}
void FAnimationEditor::FillExportAssetMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.BeginSection("AnimationExport", LOCTEXT("ExportAssetMenuHeading", "Export"));
	{
		MenuBuilder.AddMenuEntry(FAnimationEditorCommands::Get().ExportToFBX_AnimData);
		MenuBuilder.AddMenuEntry(FAnimationEditorCommands::Get().ExportToFBX_PreviewMesh);
	}
	MenuBuilder.EndSection();
}

FRichCurve* FindOrAddCurve(UCurveTable* CurveTable, FName CurveName)
{
	// Grab existing curve (if present)
	FRichCurve* Curve = CurveTable->FindRichCurve(CurveName, FString());
	if (Curve == nullptr)
	{
		// Or allocate new curve
		Curve = &CurveTable->AddRichCurve(CurveName);
	}

	return Curve;
}

void FAnimationEditor::CopyCurveToSoundWave(const FAssetData& SoundWaveAssetData) const
{
	USoundWave* SoundWave = Cast<USoundWave>(SoundWaveAssetData.GetAsset());
	UAnimSequence* Sequence = Cast<UAnimSequence>(AnimationAsset);

	if (!SoundWave || !Sequence)
	{
		return;
	}

	// If no internal table, create one now
	if (!SoundWave->GetInternalCurveData())
	{
		static const FName InternalCurveTableName("InternalCurveTable");
		UCurveTable* NewCurves = NewObject<UCurveTable>(SoundWave, InternalCurveTableName);
		NewCurves->ClearFlags(RF_Public);
		NewCurves->SetFlags(NewCurves->GetFlags() | RF_Standalone | RF_Transactional);
		SoundWave->SetCurveData(NewCurves);
		SoundWave->SetInternalCurveData(NewCurves);
	}

	UCurveTable* CurveTable = SoundWave->GetInternalCurveData();

	// iterate over curves in anim data
	for (const FFloatCurve& FloatCurve : Sequence->GetDataModel()->GetFloatCurves())
	{
		FRichCurve* Curve = FindOrAddCurve(CurveTable, FloatCurve.GetName());
		*Curve = FloatCurve.FloatCurve; // copy data
	}

	// we will need to add a curve to tell us the time we want to start playing audio
	float PreRollTime = 0.f;
	static const FName AudioCurveName("Audio");
	FRichCurve* AudioCurve = FindOrAddCurve(CurveTable, AudioCurveName);
	AudioCurve->Reset();
	AudioCurve->AddKey(PreRollTime, 1.0f);

	// Mark dirty after 
	SoundWave->MarkPackageDirty();

	FNotificationInfo Notification(FText::Format(LOCTEXT("AddedClassSuccessNotification", "Copied curves to {0}"),  FText::FromString(SoundWave->GetName())));
	FSlateNotificationManager::Get().AddNotification(Notification);

	// Close menu after picking sound
	FSlateApplication::Get().DismissAllMenus();
}

void FAnimationEditor::ConditionalRefreshEditor(UObject* InObject)
{
	bool bInterestingAsset = true;

	if (InObject != GetPersonaToolkit()->GetSkeleton() && (GetPersonaToolkit()->GetSkeleton() && InObject != GetPersonaToolkit()->GetSkeleton()->GetPreviewMesh()) && Cast<UAnimationAsset>(InObject) != AnimationAsset)
	{
		bInterestingAsset = false;
	}

	// Check that we aren't a montage that uses an incoming animation
	if (UAnimMontage* Montage = Cast<UAnimMontage>(AnimationAsset))
	{
		for (FSlotAnimationTrack& Slot : Montage->SlotAnimTracks)
		{
			if (bInterestingAsset)
			{
				break;
			}

			for (FAnimSegment& Segment : Slot.AnimTrack.AnimSegments)
			{
				if (Segment.GetAnimReference() == InObject)
				{
					bInterestingAsset = true;
					break;
				}
			}
		}
	}

	if (GetPersonaToolkit()->GetSkeleton() == nullptr)
	{
		bInterestingAsset = false;
	}

	if (bInterestingAsset)
	{
		GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
		OpenNewAnimationDocumentTab(Cast<UAnimationAsset>(InObject));
	}
}

void FAnimationEditor::HandlePostReimport(UObject* InObject, bool bSuccess)
{
	if (bSuccess)
	{
		ConditionalRefreshEditor(InObject);
	}
}

void FAnimationEditor::HandlePostImport(UFactory* InFactory, UObject* InObject)
{
	ConditionalRefreshEditor(InObject);
}

void FAnimationEditor::HandleAnimationSequenceBrowserCreated(const TSharedRef<IAnimationSequenceBrowser>& InSequenceBrowser)
{
	SequenceBrowser = InSequenceBrowser;
}

bool FAnimationEditor::RecordMeshToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset) const
{
	ISequenceRecorder& RecorderModule = FModuleManager::Get().LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
	return RecorderModule.RecordSingleNodeInstanceToAnimation(PreviewComponent, NewAsset, /*bShowMessage*/false);
}

#undef LOCTEXT_NAMESPACE
