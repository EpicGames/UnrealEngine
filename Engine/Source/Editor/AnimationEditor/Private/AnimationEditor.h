// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IAnimationEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"

class IDetailLayoutBuilder;
class FExtender;
class FMenuBuilder;
class FReferenceCollector;
class IAnimSequenceCurveEditor;
class IAnimationSequenceBrowser;
class ISkeletonTreeItem;
class ITimeSliderController;
class SDockTab;
class SWidget;
class UAnimSequence;
class UAnimSequenceBase;
class UAnimationAsset;
class UObject;
class USkeletalMeshComponent;
struct FAssetData;
struct FToolMenuContext;

namespace AnimationEditorModes
{
	// Mode identifiers
	extern const FName AnimationEditorMode;
}

namespace AnimationEditorTabs
{
	// Tab identifiers
	extern const FName DetailsTab;
	extern const FName SkeletonTreeTab;
	extern const FName ViewportTab;
	extern const FName AdvancedPreviewTab;
	extern const FName DocumentTab;
	extern const FName CurveEditorTab;
	extern const FName AssetBrowserTab;
	extern const FName AssetDetailsTab;
	extern const FName CurveNamesTab;
	extern const FName SlotNamesTab;
	extern const FName AnimMontageSectionsTab;
	extern const FName FindReplaceTab;
}

class FAnimationEditor : public IAnimationEditor, public FGCObject, public FTickableEditorObject
{
public:
	virtual ~FAnimationEditor();

	/** Edits the specified Skeleton object */
	void InitAnimationEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, class UAnimationAsset* InAnimationAsset);

	/** IAnimationEditor interface */
	virtual void SetAnimationAsset(UAnimationAsset* AnimAsset) override;
	virtual IAnimationSequenceBrowser* GetAssetBrowser() const override;
	virtual void EditCurves(UAnimSequenceBase* InAnimSequence, const TArray<FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController) override;
	virtual void StopEditingCurves(const TArray<FCurveEditInfo>& InCurveInfo) override;
	virtual FOnAnimationEditorObjectsSelected& OnAnimationEditorObjectsSelected() override { return OnAnimationEditorObjectsSelectedDelegate; }

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }

	/** IToolkit interface */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	/** FTickableEditorObject Interface */
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("AnimatingObjects/SkeletalMeshAnimation/Persona/Modes/Animation"));
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimationEditor");
	}

	/** Get the skeleton tree widget */
	TSharedRef<class ISkeletonTree> GetSkeletonTree() const { return SkeletonTree.ToSharedRef(); }

	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);

	UObject* HandleGetAsset();

	void HandleOpenNewAsset(UObject* InNewAsset);

	void HandleAnimationSequenceBrowserCreated(const TSharedRef<class IAnimationSequenceBrowser>& InSequenceBrowser);

	void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	void HandleObjectSelected(UObject* InObject);

	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	// FAssetEditorToolkit interface
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget) override;

private:

	/** Options for asset export */
	enum class EExportSourceOption : uint8
	{
		CurrentAnimation_AnimData,
		CurrentAnimation_PreviewMesh,
		Max
	};

	void HandleSectionsChanged();

	bool HasValidAnimationSequence() const;

	bool CanSetKey() const;

	void OnSetKey();

	void OnReimportAnimation(bool bWithDialog);

	void OnApplyCompression();

	void OnExportToFBX(const EExportSourceOption Option);
	//Return true mean the asset was exported, false it was cancel or it fail
	bool ExportToFBX(const TArray<UObject*> NewAssets, bool bRecordAnimation);

	void OnAddLoopingInterpolation();
	void OnRemoveBoneTrack();

	TSharedRef< SWidget > GenerateExportAssetMenu() const;

	void FillExportAssetMenu(FMenuBuilder& MenuBuilder) const;

	void CopyCurveToSoundWave(const FAssetData& SoundWaveAssetData) const;

	void ConditionalRefreshEditor(UObject* InObject);

	void HandlePostReimport(UObject* InObject, bool bSuccess);

	void HandlePostImport(class UFactory* InFactory, UObject* InObject);

private:
	void ExtendMenu();

	void ExtendToolbar();

	void BindCommands();

	TSharedPtr<SDockTab> OpenNewAnimationDocumentTab(UAnimationAsset* InAnimAsset);

	bool RecordMeshToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset) const;

	static TSharedPtr<FAnimationEditor> GetAnimationEditor(const FToolMenuContext& InMenuContext);

	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const;
	
public:
	/** Multicast delegate fired on global undo/redo */
	FSimpleMulticastDelegate OnLODChanged;

	/** Multicast delegate fired on sections changing */
	FSimpleMulticastDelegate OnSectionsChanged;

	/** Multicast delegate fired on selecting objects */
	FOnAnimationEditorObjectsSelected OnAnimationEditorObjectsSelectedDelegate;

private:
	/** The animation asset we are editing */
	TObjectPtr<UAnimationAsset> AnimationAsset;

	/** Toolbar extender */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;

	/** Persona toolkit */
	TSharedPtr<class IPersonaToolkit> PersonaToolkit;

	/** Skeleton tree */
	TSharedPtr<class ISkeletonTree> SkeletonTree;

	/** Viewport */
	TSharedPtr<class IPersonaViewport> Viewport;

	/** Details panel */
	TSharedPtr<class IDetailsView> DetailsView;

	/** The animation document currently being edited */
	TWeakPtr<SDockTab> SharedAnimDocumentTab;

	/** The animation document's curves that are currently being edited */
	TWeakPtr<SDockTab> AnimCurveDocumentTab;

	/** Sequence Browser **/
	TWeakPtr<class IAnimationSequenceBrowser> SequenceBrowser;

	/** The anim sequence curve editor */
	TWeakPtr<IAnimSequenceCurveEditor> CurveEditor;
};
