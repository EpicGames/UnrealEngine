// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AnimationEditorCommands"

void FAnimationEditorCommands::RegisterCommands()
{
	UI_COMMAND(ImportAnimation, "Import Animation", "Import new animation for the skeleton.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAnimation, "Reimport Animation", "Reimport current animation.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAnimationWithDialog, "Reimport Animation With Dialog", "Reimport current animation and show the re-import dialog.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyCompression, "Apply Compression", "Apply compression to current animation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportToFBX_AnimData, "Animation Data", "Export current animation source data to FBX", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportToFBX_PreviewMesh, "Preview Mesh", "Export current animation by playing on the current preview mesh, including retargeting, post process graph, or anything you see on the preview mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddLoopingInterpolation, "Add Looping Interpolation", "Add an extra first frame at the end of the animation to create interpolation when looping", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetKey, "Set Key", "Add Bone Transform to Additive Layer Tracks", EUserInterfaceActionType::Button, FInputChord(EKeys::S));	
	UI_COMMAND(RemoveBoneTracks, "Remove All Bone Tracks", "Remove all bone tracks. This doesn't remove notifies or curves.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
