// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FAnimationEditorCommands : public TCommands<FAnimationEditorCommands>
{
public:
	FAnimationEditorCommands()
		: TCommands<FAnimationEditorCommands>(TEXT("AnimationEditor"), NSLOCTEXT("Contexts", "AnimationEditor", "Animation Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:
	// import animation
	TSharedPtr<FUICommandInfo> ImportAnimation;
	// reimport animation
	TSharedPtr<FUICommandInfo> ReimportAnimation;
	TSharedPtr<FUICommandInfo> ReimportAnimationWithDialog;
	// apply compression
	TSharedPtr<FUICommandInfo> ApplyCompression;
	// export to FBX
	TSharedPtr<FUICommandInfo> ExportToFBX_AnimData;
	// export to FBX
	TSharedPtr<FUICommandInfo> ExportToFBX_PreviewMesh;
	// Add looping interpolation
	TSharedPtr<FUICommandInfo> AddLoopingInterpolation;
	// set key for bone track
	TSharedPtr<FUICommandInfo> SetKey;
	// Remove bone tracks
	TSharedPtr<FUICommandInfo> RemoveBoneTracks;
};
