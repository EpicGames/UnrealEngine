// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorMode.h"

#include "AnimationEditor.h"
#include "Delegates/Delegate.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "IPersonaToolkit.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "Modules/ModuleManager.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "Types/SlateEnums.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "AnimAssetFindReplace.h"

FAnimationEditorMode::FAnimationEditorMode(TSharedRef<FWorkflowCentricApplication> InHostingApp, TSharedRef<ISkeletonTree> InSkeletonTree)
	: FApplicationMode(AnimationEditorModes::AnimationEditorMode)
{
	FWorkflowAllowedTabSet& TabFactories = ApplicationModeTabFactories;

	TSharedRef<FAnimationEditor> AnimationEditor = StaticCastSharedRef<FAnimationEditor>(InHostingApp);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TabFactories.RegisterFactory(SkeletonEditorModule.CreateSkeletonTreeTabFactory(InHostingApp, InSkeletonTree));

	FOnObjectsSelected OnObjectsSelected = FOnObjectsSelected::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleObjectsSelected);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleDetailsCreated)));

	FPersonaViewportArgs ViewportArgs(AnimationEditor->GetPersonaToolkit()->GetPreviewScene());
	ViewportArgs.bShowTimeline = false;
	ViewportArgs.ContextName = TEXT("AnimationEditor.Viewport");

	PersonaModule.RegisterPersonaViewportTabFactories(TabFactories, InHostingApp, ViewportArgs);

	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, AnimationEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimationAssetBrowserTabFactory(InHostingApp, AnimationEditor->GetPersonaToolkit(), FOnOpenNewAsset::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleOpenNewAsset), FOnAnimationSequenceBrowserCreated::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleAnimationSequenceBrowserCreated), true));
	TabFactories.RegisterFactory(PersonaModule.CreateAssetDetailsTabFactory(InHostingApp, FOnGetAsset::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleGetAsset), FOnDetailsCreated()));
	TabFactories.RegisterFactory(PersonaModule.CreateCurveViewerTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), AnimationEditor->GetPersonaToolkit()->GetPreviewScene(), OnObjectsSelected));
	TabFactories.RegisterFactory(PersonaModule.CreateSkeletonSlotNamesTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), FOnObjectSelected::CreateSP(&AnimationEditor.Get(), &FAnimationEditor::HandleObjectSelected)));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimNotifiesTabFactory(InHostingApp, InSkeletonTree->GetEditableSkeleton(), OnObjectsSelected));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimMontageSectionsTabFactory(InHostingApp, AnimationEditor->GetPersonaToolkit(), AnimationEditor->OnSectionsChanged));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimAssetFindReplaceTabFactory(InHostingApp, FAnimAssetFindReplaceConfig()));
	TabFactories.RegisterFactory(PersonaModule.CreateAttributeViewerTabFactory(InHostingApp, AnimationEditor->GetPersonaToolkit()->GetPreviewScene()));

	TabLayout = FTabManager::NewLayout("Standalone_AnimationEditor_Layout_v1.5")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->AddTab(AnimationEditorTabs::SkeletonTreeTab, ETabState::OpenedTab)
						->AddTab(AnimationEditorTabs::AssetDetailsTab, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.6f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->SetHideTabWell(true)
						->AddTab(AnimationEditorTabs::ViewportTab, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.4f)
						->SetHideTabWell(true)
						->AddTab(AnimationEditorTabs::DocumentTab, ETabState::ClosedTab)
						->AddTab(AnimationEditorTabs::CurveEditorTab, ETabState::ClosedTab)
						->AddTab(AnimationEditorTabs::FindReplaceTab, ETabState::ClosedTab)
						->SetForegroundTab(AnimationEditorTabs::DocumentTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->SetHideTabWell(false)
						->AddTab(AnimationEditorTabs::DetailsTab, ETabState::OpenedTab)
						->AddTab(AnimationEditorTabs::AdvancedPreviewTab, ETabState::OpenedTab)
						->SetForegroundTab(AnimationEditorTabs::DetailsTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.4f)
						->SetHideTabWell(false)
						->AddTab(AnimationEditorTabs::AssetBrowserTab, ETabState::OpenedTab)
						->AddTab(AnimationEditorTabs::AnimMontageSectionsTab, ETabState::ClosedTab)
						->AddTab(AnimationEditorTabs::CurveNamesTab, ETabState::ClosedTab)
						->AddTab(AnimationEditorTabs::SlotNamesTab, ETabState::ClosedTab)
					)
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FAnimationEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FWorkflowCentricApplication> HostingApp = GetHost())
	{
		TSharedRef<FTabManager> TabManager = InTabManager.ToSharedRef();
		HostingApp->RegisterTabSpawners(TabManager);
		RegisterTabFactoriesWithAppAndManager(HostingApp.Get(), TabManager);
	}
}
