// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FTabManager;

class FAnimationEditorMode : public FApplicationMode
{
public:
	FAnimationEditorMode(TSharedRef<class FWorkflowCentricApplication> InHostingApp, TSharedRef<class ISkeletonTree> InSkeletonTree);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
};
