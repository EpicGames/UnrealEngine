// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditor.h"
#include "Containers/Array.h"
#include "IAnimationEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"

class IAnimationEditor;
class IToolkitHost;
class UAnimationAsset;

class FAnimationEditorModule : public IAnimationEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FAnimationEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
	}

	virtual TSharedRef<IAnimationEditor> CreateAnimationEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimationAsset* InAnimationAsset) override
	{
		TSharedRef<FAnimationEditor> AnimationEditor(new FAnimationEditor());
		AnimationEditor->InitAnimationEditor(Mode, InitToolkitHost, InAnimationAsset);
		return AnimationEditor;
	}

	virtual TArray<FAnimationEditorToolbarExtender>& GetAllAnimationEditorToolbarExtenders() override { return AnimationEditorToolbarExtenders; }

	/** Gets the extensibility managers for outside entities to extend this editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TArray<FAnimationEditorToolbarExtender> AnimationEditorToolbarExtenders;
};

IMPLEMENT_MODULE(FAnimationEditorModule, AnimationEditor);
