// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolMenuContext.h"
#include "AnimationToolMenuContext.generated.h"

class IAnimationEditor;

UCLASS(MinimalAPI)
class UAnimationToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<IAnimationEditor> AnimationEditor;
};
