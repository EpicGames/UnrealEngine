// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PersonaAssetEditorToolkit.h"
#include "IHasPersonaToolkit.h"

class UAnimationAsset;
class IAnimationSequenceBrowser;
class UAnimSequenceBase;
struct FRichCurve;
class ITimeSliderController;
enum class ERawCurveTrackTypes : uint8;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAnimationEditorObjectsSelected, const TArray<UObject*>& /*InObjects*/);

class IAnimationEditor : public FPersonaAssetEditorToolkit, public IHasPersonaToolkit
{
public:
	/** Set the animation asset of the editor. */
	virtual void SetAnimationAsset(UAnimationAsset* AnimAsset) = 0;

	/** Get the asset browser we host */
	virtual IAnimationSequenceBrowser* GetAssetBrowser() const = 0;

	/** Support structure for EditCurves */
	struct FCurveEditInfo
	{
		FCurveEditInfo(const FText& InCurveDisplayName, const FLinearColor& InCurveColor, const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, FSimpleDelegate OnCurveModified = FSimpleDelegate())
			: CurveDisplayName(InCurveDisplayName)
			, CurveColor(InCurveColor)
			, CurveName(InName)
			, Type(InType)
			, CurveIndex(InCurveIndex)
			, OnCurveModified(OnCurveModified)
		{}

		FCurveEditInfo(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex)
			: CurveName(InName)
			, Type(InType)
			, CurveIndex(InCurveIndex)
		{}

		bool operator==(const FCurveEditInfo& InCurveEditInfo) const
		{
			return CurveName == InCurveEditInfo.CurveName && Type == InCurveEditInfo.Type && CurveIndex == InCurveEditInfo.CurveIndex;
		}

		FText CurveDisplayName;
		FLinearColor CurveColor;
		FName CurveName;
		ERawCurveTrackTypes Type;
		int32 CurveIndex;
		FSimpleDelegate OnCurveModified;
	};

	/** Edit the specified curves on the specified sequence */
	virtual void EditCurves(UAnimSequenceBase* InAnimSequence, const TArray<FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController) = 0;

	/** Stop editing the specified curves */
	virtual void StopEditingCurves(const TArray<FCurveEditInfo>& InCurveInfo) = 0;

	/** Multicast delegate fired on selecting objects */
	virtual FOnAnimationEditorObjectsSelected& OnAnimationEditorObjectsSelected() = 0;
};
