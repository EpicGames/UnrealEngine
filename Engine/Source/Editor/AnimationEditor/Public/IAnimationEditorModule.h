// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IAnimationEditor.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAnimationEditor, Log, All);

class IAnimationEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new skeleton editor instance */
	virtual TSharedRef<IAnimationEditor> CreateAnimationEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, class UAnimationAsset* InAnimationAsset) = 0;

	/** Get all toolbar extenders */
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FAnimationEditorToolbarExtender, const TSharedRef<FUICommandList> /*InCommandList*/, TSharedRef<IAnimationEditor> /*InAnimationEditor*/);
	virtual TArray<FAnimationEditorToolbarExtender>& GetAllAnimationEditorToolbarExtenders() = 0;
};
